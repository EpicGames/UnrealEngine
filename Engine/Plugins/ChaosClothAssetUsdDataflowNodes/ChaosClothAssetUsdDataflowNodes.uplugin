{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Cloth Asset USD Dataflow Nodes",
	"Description": "Dataflow nodes for importing USD Cloth Assets.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosClothAssetUsdDataflowNodes",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		}
	],
	"Plugins": [
		{
			"Name": "ChaosClothAsset",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAssetDataflowNodes",
			"Enabled": true
		},
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "USDImporter",
			"Enabled": true,
			"SupportedTargetPlatforms": [ "Win64", "Mac", "Linux" ]
		}
	]
}
