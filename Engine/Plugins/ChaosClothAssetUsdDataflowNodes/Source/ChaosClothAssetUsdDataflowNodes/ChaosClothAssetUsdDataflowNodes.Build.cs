// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using UnrealBuildTool;

public class ChaosClothAssetUsdDataflowNodes : ModuleRules
{
	public ChaosClothAssetUsdDataflowNodes(ReadOnlyTargetRules Target) : base(Target)
	{	
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetTools",
				"Chaos",
				"Core",
				"CoreUObject",
				"ChaosClothAsset",
				"ChaosClothAssetDataflowNodes",
				"ChaosClothAssetEngine",
				"ChaosClothAssetTools",
				"DataflowCore",
				"DataflowEditor",
				"DataflowEngine",
				"DesktopWidgets",  // For SFilePathPicker
				"Engine",
				"MeshDescription",
				"SlateCore",
				"StaticMeshDescription",
				"UnrealEd",
				"UnrealUSDWrapper",
				"USDClasses",	
				"USDSchemas",
				"USDStage",
				"USDStageImporter",
				"USDUtilities",
				// ... add private dependencies that you statically link with here ...
			}
		);

		PrivateIncludePaths.Add(System.IO.Path.Combine(GetModuleDirectory("ChaosClothAssetDataflowNodes"), "Internal"));
	}
}
