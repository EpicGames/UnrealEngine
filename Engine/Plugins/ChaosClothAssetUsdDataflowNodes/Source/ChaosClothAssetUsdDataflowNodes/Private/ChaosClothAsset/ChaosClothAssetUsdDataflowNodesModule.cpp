// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ColorScheme.h"
#include "ChaosClothAsset/ImportFilePathCustomization.h"
#include "ChaosClothAsset/USDImportNode_v3.h"
#include "Dataflow/DataflowCategoryRegistry.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		static void RegisterDataflowNodes()
		{
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Cloth", FColorScheme::NodeHeader, FColorScheme::NodeBody);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetUSDImportNode_v3);
			// Deprecated nodes
PRAGMA_DISABLE_DEPRECATION_WARNINGS
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}  // End namespace Private

	class FChaosClothAssetUsdDataflowNodesModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			using namespace UE::Chaos::ClothAsset;

			Private::RegisterDataflowNodes();

			UE_DATAFLOW_REGISTER_CATEGORY_FORASSET_TYPE("Cloth", UChaosClothAsset);

			// Register type customizations
			if (FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This structure will be made private.")
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetImportFilePath::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FImportFilePathCustomization::MakeInstance));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}

		virtual void ShutdownModule() override
		{
			// Unregister type customizations
			if (UObjectInitialized() && !IsEngineExitRequested())
			{
				if (FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This structure will be made private.")
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetImportFilePath::StaticStruct()->GetFName());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
		}
	};
}  // End namespace UE::Chaos::ClothAsset

IMPLEMENT_MODULE(UE::Chaos::ClothAsset::FChaosClothAssetUsdDataflowNodesModule, ChaosClothAssetUsdDataflowNodes)
