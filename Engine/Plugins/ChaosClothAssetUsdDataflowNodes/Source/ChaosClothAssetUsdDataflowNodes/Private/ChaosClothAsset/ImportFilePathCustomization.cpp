// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ImportFilePathCustomization.h"
#include "ChaosClothAsset/ImportFilePath.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SFilePathPicker.h"
#include "DetailWidgetRow.h"
#include "EditorDirectories.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "ImportFilePathCustomization"

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		static const FChaosClothAssetImportFilePath* GetImportFilePath(const TSharedPtr<IPropertyHandle>& PropertyHandle)
		{
			const FChaosClothAssetImportFilePath* ImportFilePath = nullptr;
			if (PropertyHandle)
			{
				if (const FStructProperty* const StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
				{
					if (StructProperty->Struct && StructProperty->Struct->IsChildOf(FChaosClothAssetImportFilePath::StaticStruct()))
					{
						void* Data;
						if (PropertyHandle->GetValueData(Data) == FPropertyAccess::Success)
						{
							ImportFilePath = reinterpret_cast<const FChaosClothAssetImportFilePath*>(Data);
						}
					}
				}
			}
			return ImportFilePath;
		}
	}

	TSharedRef<IPropertyTypeCustomization> FImportFilePathCustomization::MakeInstance()
	{
		return MakeShareable(new FImportFilePathCustomization);
	}

	void FImportFilePathCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		// Keep a weak pointer to the graph editor asking for this customization
		DataflowGraphEditor = SDataflowGraphEditor::GetSelectedGraphEditor();

		StructProperty = StructPropertyHandle;
		PathStringProperty = StructPropertyHandle->GetChildHandle(TEXT("FilePath"));

		// Construct file type filter
		FString FileTypeFilter;

		const FString& MetaData = StructPropertyHandle->GetMetaData(TEXT("FilePathFilter"));

		bLongPackageName = StructPropertyHandle->HasMetaData(TEXT("LongPackageName"));
		bRelativeToGameDir = StructPropertyHandle->HasMetaData(TEXT("RelativeToGameDir"));

		if (MetaData.IsEmpty())
		{
			FileTypeFilter = TEXT("All files (*.*)|*.*");
		}
		else
		{
			if (MetaData.Contains(TEXT("|"))) // If MetaData follows the Description|ExtensionList format, use it as is
			{
				FileTypeFilter = MetaData;
			}
			else
			{
				FileTypeFilter = FString::Printf(TEXT("%s files (*.%s)|*.%s"), *MetaData, *MetaData, *MetaData);
			}
		}

		// Create path picker widget
		HeaderRow
			.NameContent()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(250)
			.MaxDesiredWidth(0)
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(SFilePathPicker)
							.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
							.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							.BrowseButtonToolTip(LOCTEXT("FileButtonToolTipText", "Choose a file from this computer"))
							.BrowseDirectory(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN))
							.BrowseTitle(LOCTEXT("PropertyEditorTitle", "File picker..."))
							.FilePath(this, &FImportFilePathCustomization::HandleFilePathPickerFilePath)
							.FileTypeFilter(FileTypeFilter)
							.OnPathPicked(this, &FImportFilePathCustomization::HandleFilePathPickerPathPicked)
					]
			];
	}

	// Callbacks lifted from FFilePathStructCustomization
	FString FImportFilePathCustomization::HandleFilePathPickerFilePath() const
	{
		FString FilePath;
		if (PathStringProperty)
		{
			PathStringProperty->GetValue(FilePath);
		}
		return FilePath;
	}

	void FImportFilePathCustomization::HandleFilePathPickerPathPicked(const FString& PickedPath)
	{
		FString FinalPath = PickedPath;
		if (bLongPackageName)
		{
			FString LongPackageName;
			FString StringFailureReason;
			if (FPackageName::TryConvertFilenameToLongPackageName(PickedPath, LongPackageName, &StringFailureReason) == false)
			{
				FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(StringFailureReason));
			}
			FinalPath = LongPackageName;
		}
		else if (bRelativeToGameDir && !PickedPath.IsEmpty())
		{
			//A filepath under the project directory will be made relative to the project directory
			//Otherwise, the absolute path will be returned unless it doesn't exist, the current path will
			//be kept. This can happen if it's already relative to project dir (tabbing when selected)

			const FString ProjectDir = FPaths::ProjectDir();
			const FString AbsoluteProjectDir = FPaths::ConvertRelativePathToFull(ProjectDir);
			const FString AbsolutePickedPath = FPaths::ConvertRelativePathToFull(PickedPath);

			//Verify if absolute path to file exists. If it was already relative to content directory
			//the absolute will be to binaries and will possibly be garbage
			if (FPaths::FileExists(AbsolutePickedPath))
			{
				//If file is part of the project dir, chop the project dir part
				//Otherwise, use the absolute path
				if (AbsolutePickedPath.StartsWith(AbsoluteProjectDir))
				{
					FinalPath = AbsolutePickedPath.RightChop(AbsoluteProjectDir.Len());
				}
				else
				{
					FinalPath = AbsolutePickedPath;
				}
			}
			else
			{
				//If absolute file doesn't exist, it might already be relative to project dir
				//If not, then it might be a manual entry, so keep it untouched either way
				FinalPath = PickedPath;
			}
		}

		if (PathStringProperty)
		{
			// The value can be set twice from pressing enter and losing the focus, most likely triggering two reentrant evaluations if not avoided
			FString OldPath;
			PathStringProperty->GetValue(OldPath);
			if (OldPath != FinalPath)
			{
				PathStringProperty->SetValue(FinalPath, EPropertyValueSetFlags::InteractiveChange);  // Do as an interactive change so that the delegate is called with the correct value before the node invalidates
				if (const FChaosClothAssetImportFilePath* const ImportFilePath = Private::GetImportFilePath(StructProperty))
				{
					// Retrieve context if any
					const TSharedPtr<const SDataflowGraphEditor> DataflowGraphEditorPtr = DataflowGraphEditor.Pin();
					const TSharedPtr<UE::Dataflow::FContext> Context = DataflowGraphEditorPtr ? DataflowGraphEditorPtr->GetDataflowContext() : TSharedPtr<UE::Dataflow::FContext>();

					// Execute function
					UE::Dataflow::FContextThreaded EmptyContext;
					ImportFilePath->Execute(Context.IsValid() ? *Context : EmptyContext);
				}
				PathStringProperty->SetValue(FinalPath);  // This will set the final value and call invalidate on the node
			}
		}
		FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(PickedPath));
	}
}  // End namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
