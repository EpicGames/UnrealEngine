// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "IPropertyTypeCustomization.h"

class SButton;
class SDataflowGraphEditor;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Customization for import file path, modelled after FFilePathStructCustomization with the addition 
	 * of a reimport button whenever a bForceReimport property is present in the customized struct.
	 */
	class FImportFilePathCustomization : public IPropertyTypeCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> /*StructPropertyHandle*/, IDetailChildrenBuilder& /*ChildBuilder*/, IPropertyTypeCustomizationUtils& /*CustomizationUtils*/) override {}

	private:
		FReply OnClicked();

		FString HandleFilePathPickerFilePath() const;
		void HandleFilePathPickerPathPicked(const FString& PickedPath);

		TWeakPtr<const SDataflowGraphEditor> DataflowGraphEditor;
		TSharedPtr<IPropertyHandle> StructProperty;
		TSharedPtr<IPropertyHandle> PathStringProperty;
		bool bLongPackageName;
		bool bRelativeToGameDir;
	};
}
