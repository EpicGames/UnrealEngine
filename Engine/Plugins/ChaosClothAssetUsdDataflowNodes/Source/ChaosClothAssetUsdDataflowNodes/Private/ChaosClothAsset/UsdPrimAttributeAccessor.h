// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if USE_USD_SDK
#include "USDStageOptions.h"
#include "USDValueConversion.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/VtValue.h"

namespace UE::Chaos::ClothAsset
{
	class FUsdPrimAttributeAccessor
	{
	public:
		explicit FUsdPrimAttributeAccessor(const FUsdPrim& InUsdPrim, const EUsdUpAxis UsdUpAxis = EUsdUpAxis::ZAxis) 
			: UsdPrim(InUsdPrim)
			, AxesOrder(
				0,
				(UsdUpAxis == EUsdUpAxis::ZAxis) ? 1 : 2,
				(UsdUpAxis == EUsdUpAxis::ZAxis) ? 2 : 1)
		{
		}

		template<typename ValueType>
		ValueType GetValue(const TCHAR* AttributeName, const ValueType DefaultValue = ValueType(0)) const
		{
			if (UsdPrim.HasAttribute(AttributeName))
			{
				const FUsdAttribute UsdAttribute = UsdPrim.GetAttribute(AttributeName);
				if (UsdAttribute.HasValue())
				{
					FVtValue VtValue;
					UsdAttribute.Get(VtValue);
					return GetValueAs<ValueType>(VtValue, DefaultValue);
				}
			}
			return DefaultValue;
		}

		template<typename ValueType>
		TArray<ValueType> GetArray(const TCHAR* AttributeName) const
		{
			if (UsdPrim.HasAttribute(AttributeName))
			{
				const FUsdAttribute UsdAttribute = UsdPrim.GetAttribute(AttributeName);
				if (UsdAttribute.HasValue())
				{
					FVtValue VtValue;
					UsdAttribute.Get(VtValue);
					return GetArrayAs<ValueType>(VtValue);
				}
			}
			return TArray<ValueType>();
		}

		template<typename ValueType>
		ValueType GetArrayValue(const TCHAR* AttributeName, const ValueType DefaultValue = ValueType(0), const int32 ValueIndex = 0) const
		{
			const TArray<ValueType> Array = GetArray<ValueType>(AttributeName);
			return Array.IsValidIndex(ValueIndex) ? Array[ValueIndex] : DefaultValue;
		}

	private:
		template<typename ValueType>
		ValueType GetValueAs(const FVtValue& VtValue, const ValueType DefaultValue = ValueType(0)) const
		{
			unimplemented();
			return DefaultValue;
		}

		template<typename ValueType>
		TArray<ValueType> GetArrayAs(const FVtValue& VtValue) const
		{
			unimplemented();
			return TArray<ValueType>();
		}

		const FUsdPrim UsdPrim;
		const FIntVector AxesOrder;
	};

	template<>
	inline uint32 FUsdPrimAttributeAccessor::GetValueAs<uint32>(const FVtValue& VtValue, const uint32 DefaultValue) const
	{
		return VtValue.GetTypeName() == TEXT("unsigned int") ? UsdUtils::GetUnderlyingValue<uint32>(VtValue).Get(DefaultValue) : DefaultValue;  // Note: FUsdAttribute.GetTypeName() would return "uint" instead!
	}

	template<>
	inline float FUsdPrimAttributeAccessor::GetValueAs<float>(const FVtValue& VtValue, const float DefaultValue) const
	{
		return VtValue.GetTypeName() == TEXT("float") ? UsdUtils::GetUnderlyingValue<float>(VtValue).Get(DefaultValue) : DefaultValue;
	}

	template<>
	inline FVector3f FUsdPrimAttributeAccessor::GetValueAs<FVector3f>(const FVtValue& VtValue, const FVector3f DefaultValue) const
	{
		if (VtValue.GetTypeName() == TEXT("GfVec3f"))  // Note: FUsdAttribute.GetTypeName() would return "float3" instead!
		{
			UsdUtils::FConvertedVtValue ConvertedVtValue;
			if (UsdToUnreal::ConvertValue(VtValue, ConvertedVtValue) && !ConvertedVtValue.bIsArrayValued && !ConvertedVtValue.bIsEmpty)
			{
				return FVector3f(
					ConvertedVtValue.Entries[0][AxesOrder[0]].Get<float>(),
					ConvertedVtValue.Entries[0][AxesOrder[1]].Get<float>(),
					ConvertedVtValue.Entries[0][AxesOrder[2]].Get<float>());
			}
		}
		return DefaultValue;
	}

	template<>
	inline TArray<float> FUsdPrimAttributeAccessor::GetArrayAs<float>(const FVtValue& VtValue) const
	{
		TArray<float> Array;
		if (VtValue.GetTypeName() == TEXT("VtArray<float>"))  // Note: FUsdAttribute.GetTypeName() would return "float[]" instead!
		{
			UsdUtils::FConvertedVtValue ConvertedVtValue;
			if (UsdToUnreal::ConvertValue(VtValue, ConvertedVtValue) && ConvertedVtValue.bIsArrayValued && !ConvertedVtValue.bIsEmpty)
			{
				Array.Reserve(ConvertedVtValue.Entries.Num());
				for (int32 Index = 0; Index < ConvertedVtValue.Entries.Num(); ++Index)
				{
					Array.Emplace(ConvertedVtValue.Entries[Index][0].Get<float>());
				}
			}
		}
		return Array;
	}
}
#endif  // #if USE_USD_SDK
